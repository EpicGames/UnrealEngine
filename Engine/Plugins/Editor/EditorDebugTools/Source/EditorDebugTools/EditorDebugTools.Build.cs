// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class EditorDebugTools : ModuleRules
{
	public EditorDebugTools(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;			
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
			);			
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Projects",
				"InputCore",
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
				"SourceCodeAccess",
				"WorkspaceMenuStructure",
				"AppFramework",
			}
			);

		if (Target.Configuration != UnrealTargetConfiguration.Shipping)
		{
			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"SlateReflector",
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"SlateReflector",
				}
			);
		}
	}
}
