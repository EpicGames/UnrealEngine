// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorDebugTools.h"
#include "EditorDebugToolsStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "SDebugPanel.h"
#include "GammaUIPanel.h"
#include "SModuleUI.h"
#include "WorkspaceMenuStructureModule.h"
#include "WorkspaceMenuStructure.h"

static const FName EditorDebugToolsTabName("DebugTools");
static const FName ModulesTabName("Modules");

#define LOCTEXT_NAMESPACE "FEditorDebugToolsModule"

TSharedRef<SDockTab> CreateDebugToolsTab(const FSpawnTabArgs& Args)
{
	return
		SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SDebugPanel)
			]
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SGammaUIPanel)
			]
		];
}

TSharedRef<SDockTab> CreateModulesTab(const FSpawnTabArgs& Args)
{
	return
		SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			SNew(SBorder)
			.Padding(2)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SModuleUI)
			]
		];
}

void FEditorDebugToolsModule::StartupModule()
{
	FEditorDebugToolsStyle::Initialize();

	FTabSpawnerEntry& Spawner = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(EditorDebugToolsTabName, FOnSpawnTab::CreateStatic(&CreateDebugToolsTab))
		.SetDisplayName(NSLOCTEXT("Toolbox", "DebugTools", "Debug Tools"))
		.SetTooltipText(NSLOCTEXT("Toolbox", "DebugToolsTooltipText", "Open the Debug Tools tab."))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Debug"))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsDebugCategory());

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(ModulesTabName, FOnSpawnTab::CreateStatic(&CreateModulesTab))
		.SetDisplayName(NSLOCTEXT("Toolbox", "Modules", "Modules"))
		.SetTooltipText(NSLOCTEXT("Toolbox", "ModulesTooltipText", "Open the Modules tab."))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Modules"))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsDebugCategory());
}

void FEditorDebugToolsModule::ShutdownModule()
{
	FEditorDebugToolsStyle::Shutdown();

	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(EditorDebugToolsTabName);

	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(ModulesTabName);
}



#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FEditorDebugToolsModule, EditorDebugTools)
