// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorDebugToolsStyle.h"
#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateStyleRegistry.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/SlateStyle.h"

TSharedPtr< FSlateStyleSet > FEditorDebugToolsStyle::StyleInstance = NULL;

#define LOCAL_IMAGE_BRUSH( RelativePath, ... ) FSlateImageBrush( FPaths::EngineContentDir() / "Editor/Slate"/ RelativePath + TEXT(".png"), __VA_ARGS__ )
#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( Style->RootToContentDir( RelativePath, TEXT(".png") ), __VA_ARGS__ )

void FEditorDebugToolsStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FEditorDebugToolsStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

FName FEditorDebugToolsStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("EditorDebugToolsStyle"));
	return StyleSetName;
}


const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon40x40(40.0f, 40.0f);

TSharedRef< FSlateStyleSet > FEditorDebugToolsStyle::Create()
{
	TSharedRef< FSlateStyleSet > Style = MakeShareable(new FSlateStyleSet("EditorDebugToolsStyle"));
	Style->SetContentRoot(IPluginManager::Get().FindPlugin("EditorDebugTools")->GetBaseDir() / TEXT("Resources/Slate"));

	// Gamma reference.
	Style->Set("GammaReference", new IMAGE_PLUGIN_BRUSH("GammaReference", FVector2D(256, 128)));

	return Style;
}



const ISlateStyle& FEditorDebugToolsStyle::Get()
{
	return *StyleInstance;
}

#undef LOCAL_IMAGE_BRUSH
#undef IMAGE_PLUGIN_BRUSH
