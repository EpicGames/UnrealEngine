// Copyright Epic Games, Inc. All Rights Reserved.

#include "GammaUIPanel.h"
#include "Engine/Engine.h"
#include "Widgets/Input/SSpinBox.h"
#include "EditorDebugToolsStyle.h"

void SGammaUIPanel::Construct(const SGammaUIPanel::FArguments& InArgs)
{
	ChildSlot
	.Padding( FMargin(8) )
	[
		SNew( SVerticalBox )
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 4.0f, 0.0f, 4.0f)
		[
			SNew(STextBlock)
			.Text(NSLOCTEXT("GammaUI", "GammaUILabel", "Gamma"))
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding( 0.0f, 4.0f, 0.0f, 4.0f )
		[
			SNew( SSpinBox<float> )
			.Delta(0.01f)
			.MinValue(1.0f)
			.MaxValue(3.0f)
			.Value( this, &SGammaUIPanel::OnGetGamma )
			.OnValueChanged( this, &SGammaUIPanel::OnGammaChanged )
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.0f, 4.0f, 0.0f, 4.0f)
		.HAlign(HAlign_Center)
		[
			SNew(SImage)
			.Image(FEditorDebugToolsStyle::Get().GetBrush(TEXT("GammaReference")))
		]
	];
}

float SGammaUIPanel::OnGetGamma() const
{
	return GEngine ? GEngine->DisplayGamma : 2.2f;
}

void SGammaUIPanel::OnGammaChanged(float NewValue)
{
	if( GEngine )
	{
		GEngine->DisplayGamma = NewValue;
	}
}
