// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SGammaUIPanel : public SCompoundWidget
{

public:

	SLATE_BEGIN_ARGS( SGammaUIPanel ){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	float OnGetGamma() const;
	void OnGammaChanged( float NewValue );
};
