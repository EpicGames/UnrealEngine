// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDebugPanel.h"
#include "Rendering/SlateRenderer.h"
#include "Widgets/Testing/SStarshipSuite.h"
#include "Widgets/Testing/STestSuite.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Input/SButton.h"
#include "Modules/ModuleManager.h"
#include "Framework/Application/SlateApplication.h"

#if !UE_BUILD_SHIPPING
#include "ISlateReflectorModule.h"
#endif

void SDebugPanel::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew( SVerticalBox )
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding( 4.0f )
		.HAlign(HAlign_Left)
		[
			SNew( SButton )
			.Text( NSLOCTEXT("DeveloperToolbox", "ReloadTextures", "Reload Textures") )
			.OnClicked( this, &SDebugPanel::OnReloadTexturesClicked )
		]
			
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding( 4.0f )
		.HAlign(HAlign_Left)
		[
			SNew( SButton )
			.Text( NSLOCTEXT("DeveloperToolbox", "FlushFontCache", "Flush Font Cache") )
			.OnClicked( this, &SDebugPanel::OnFlushFontCacheClicked )
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding( 4.0f )
		.HAlign(HAlign_Left)
		[
			SNew( SButton )
			.Text( NSLOCTEXT("DeveloperToolbox", "DisplayTextureAtlases", "Display Texture Atlases") )
			.OnClicked( this, &SDebugPanel::OnDisplayTextureAtlases )
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding( 4.0f )
		.HAlign(HAlign_Left)
		[
			SNew( SButton )
			.Text( NSLOCTEXT("DeveloperToolbox", "DisplayFontAtlases", "Display Font Atlases") )
			.OnClicked( this, &SDebugPanel::OnDisplayFontAtlases )
		]
	];
}

FReply SDebugPanel::OnReloadTexturesClicked()
{
	FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
	return FReply::Handled();
}

FReply SDebugPanel::OnDisplayTextureAtlases()
{
#if !UE_BUILD_SHIPPING
	static const FName SlateReflectorModuleName("SlateReflector");
	FModuleManager::LoadModuleChecked<ISlateReflectorModule>(SlateReflectorModuleName).DisplayTextureAtlasVisualizer();
#endif // #if !UE_BUILD_SHIPPING
	return FReply::Handled();
}

FReply SDebugPanel::OnDisplayFontAtlases()
{
#if !UE_BUILD_SHIPPING
	static const FName SlateReflectorModuleName("SlateReflector");
	FModuleManager::LoadModuleChecked<ISlateReflectorModule>(SlateReflectorModuleName).DisplayFontAtlasVisualizer();
#endif // #if !UE_BUILD_SHIPPING
	return FReply::Handled();
}

FReply SDebugPanel::OnFlushFontCacheClicked()
{
	FSlateApplication::Get().GetRenderer()->FlushFontCache(TEXT("SDebugPanel::OnFlushFontCacheClicked"));
	return FReply::Handled();
}
