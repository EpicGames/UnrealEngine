// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class SDebugPanel : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDebugPanel) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:
	FReply OnReloadTexturesClicked();
	FReply OnDisplayTextureAtlases();
	FReply OnDisplayFontAtlases();
	FReply OnFlushFontCacheClicked();
	FReply OnTestSuiteClicked();
};

