// Copyright Epic Games, Inc. All Rights Reserved.

#include "SModuleUI.h"
#include "Framework/Views/TableViewMetadata.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/FeedbackContext.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Engine/Engine.h"
#include "Misc/Paths.h"
#include "SlateOptMacros.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Views/SListView.h"
#include "Misc/HotReloadInterface.h"
#include "Widgets/Input/SSearchBox.h"
#include "ISourceCodeAccessModule.h"
#include "ISourceCodeAccessor.h"

static bool CanAccessSourceCode()
{
	ISourceCodeAccessModule* SourceCodeAccessModule = FModuleManager::GetModulePtr<ISourceCodeAccessModule>("SourceCodeAccess");
	return SourceCodeAccessModule != nullptr && SourceCodeAccessModule->GetAccessor().CanAccessSourceCode();
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SModuleUI::Construct(const SModuleUI::FArguments& InArgs)
{
	#define LOCTEXT_NAMESPACE "ModuleUI"

	ChildSlot
		.Padding( FMargin(8) )
	[
		SNew( SVerticalBox )

			+ SVerticalBox::Slot()
				.AutoHeight()
			[
				SAssignNew(ModuleNameSearchBox, SSearchBox)
				.OnTextChanged(this, &SModuleUI::OnFilterTextChanged)
			]

			// List of modules
			+ SVerticalBox::Slot()
				.FillHeight( 1.0f )		// We want the list to stretch vertically to fill up the user-resizable space
			[
				SAssignNew( ModuleListView, SModuleListView )
					.ListItemsSource( &ModuleListItems )
					.OnGenerateRow( this, &SModuleUI::OnGenerateWidgetForModuleListView )
					.HeaderRow
					(
						SNew(SHeaderRow)
						+SHeaderRow::Column("ModuleName")
						.DefaultLabel(NSLOCTEXT("ModuleUI", "ModuleName", "Module"))
						.FillWidth(200)
						+SHeaderRow::Column("ModuleActions")
						.DefaultLabel(NSLOCTEXT("ModuleUI", "ModuleActions", "Actions"))
						.FillWidth(1000)
					)
			]
	];

	#undef LOCTEXT_NAMESPACE


	// Register to find out about module changes
	FModuleManager::Get().OnModulesChanged().AddSP( this, &SModuleUI::OnModulesChanged );


	// Gather data from module manager
	UpdateModuleListItems();
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


SModuleUI::~SModuleUI()
{
	// Unregister callbacks
	FModuleManager::Get().OnModulesChanged().RemoveAll( this );
}

void SModuleUI::OnFilterTextChanged(const FText& InFilterText)
{
	UpdateModuleListItems();
}
TSharedRef<ITableRow> SModuleUI::OnGenerateWidgetForModuleListView(TSharedPtr< FModuleListItem > InItem, const TSharedRef<STableViewBase>& OwnerTable)
{
	#define LOCTEXT_NAMESPACE "ModuleUI"

	class SModuleItemWidget : public SMultiColumnTableRow< TSharedPtr< FModuleListItem > >
	{
		public:
		SLATE_BEGIN_ARGS( SModuleItemWidget ){}
		SLATE_END_ARGS()

		void Construct( const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<FModuleListItem> InListItem )
		{
			Item = InListItem;
	
			SMultiColumnTableRow< TSharedPtr< FModuleListItem > >::Construct( FSuperRowType::FArguments(), InOwnerTable );			
		}

		TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName )
		{
			if ( ColumnName == "ModuleName" )
			{
				return
					SNew( STextBlock )
					.Text( FText::FromName(Item->ModuleName) );
			}
			else if ( ColumnName == "ModuleActions" )
			{
				return 
				SNew( SHorizontalBox )
	
				// Load button
				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding( 2.0f, 0.0f )
				[
					SNew( SButton )
						.Visibility( Item.ToSharedRef(), &FModuleListItem::GetVisibilityBasedOnUnloadedState )
						.Text( LOCTEXT("Load", "Load") )
						.OnClicked( Item.ToSharedRef(), &FModuleListItem::OnLoadClicked )
				]

				// Unload button
				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding( 2.0f, 0.0f )
				[
					SNew( SButton )
						.Visibility( Item.ToSharedRef(), &FModuleListItem::GetVisibilityBasedOnLoadedAndShutdownableState )
						.Text( LOCTEXT("Unload", "Unload") )
						.OnClicked( Item.ToSharedRef(), &FModuleListItem::OnUnloadClicked )
				]

				// Reload button
				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding( 2.0f, 0.0f )
					[
						SNew( SButton )
						.Visibility( Item.ToSharedRef(), &FModuleListItem::GetVisibilityBasedOnReloadableState )
						.Text( LOCTEXT("Reload", "Reload") )
						.OnClicked( Item.ToSharedRef(), &FModuleListItem::OnReloadClicked )
					]

				// Recompile button
				+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding( 2.0f, 0.0f )
				[
					SNew( SButton )
						.Visibility( Item.ToSharedRef(), &FModuleListItem::GetVisibilityBasedOnRecompilableState )
						.Text( LOCTEXT("Recompile", "Recompile") )
						.OnClicked( Item.ToSharedRef(), &FModuleListItem::OnRecompileClicked )
				];
			}
			else
			{
				return SNew(STextBlock) .Text(LOCTEXT("UnknownColumn", "Unknown Column"));
			}

		}

		TSharedPtr< FModuleListItem > Item;
	};

	return SNew( SModuleItemWidget, OwnerTable, InItem );

	#undef LOCTEXT_NAMESPACE
}


void SModuleUI::OnModulesChanged( FName ModuleThatChanged, EModuleChangeReason ReasonForChange )
{
	// @todo: Consider using dirty bit for this instead, refresh on demand
	UpdateModuleListItems();
}

void SModuleUI::UpdateModuleListItems()
{
	ModuleListItems.Reset();

	TArray< FModuleStatus > ModuleStatuses;
	FModuleManager::Get().QueryModules( ModuleStatuses );

	for( TArray< FModuleStatus >::TConstIterator ModuleIt( ModuleStatuses ); ModuleIt; ++ModuleIt )
	{
		const FModuleStatus& ModuleStatus = *ModuleIt;

		FName ModuleName(*ModuleStatus.Name);

		FString ModuleNameStr = ModuleName.ToString();
		FString FilterStr = ModuleNameSearchBox->GetText().ToString();
		FilterStr.TrimStartAndEndInline();
		if ( (!FilterStr.IsEmpty() && ModuleNameStr.Contains(FilterStr)) || 
			  FilterStr.IsEmpty() )
		{
			TSharedRef< FModuleListItem > NewItem(new FModuleListItem());
			NewItem->ModuleName = ModuleName;
			ModuleListItems.Add(NewItem);
		}
	}


	// Sort the list of modules alphabetically
	struct FModuleSorter
	{
		FORCEINLINE bool operator()( const TSharedPtr<FModuleListItem>& A, const TSharedPtr<FModuleListItem>& B ) const
		{
			return A->ModuleName.LexicalLess(B->ModuleName);
		}
	};
	ModuleListItems.Sort( FModuleSorter() );

	// Update the list view if we have one
	if( ModuleListView.IsValid() )
	{
		ModuleListView->RequestListRefresh();
	}
}


FReply SModuleUI::FModuleListItem::OnLoadClicked()
{
	GEngine->DeferredCommands.Add( FString::Printf( TEXT( "Module Load %s" ), *ModuleName.ToString() ) );
	return FReply::Handled();
}


FReply SModuleUI::FModuleListItem::OnUnloadClicked()
{
	GEngine->DeferredCommands.Add( FString::Printf( TEXT( "Module Unload %s" ), *ModuleName.ToString() ) );
	return FReply::Handled();
}


FReply SModuleUI::FModuleListItem::OnReloadClicked()
{
	GEngine->DeferredCommands.Add( FString::Printf( TEXT( "Module Reload %s" ), *ModuleName.ToString() ) );
	return FReply::Handled();
}


FReply SModuleUI::FModuleListItem::OnRecompileClicked()
{
	const bool bShowProgressDialog = true;
	const bool bShowCancelButton = false;

	FFormatNamedArguments Args;
	Args.Add( TEXT("ModuleName"), FText::FromName( ModuleName ) );
	GWarn->BeginSlowTask( FText::Format( NSLOCTEXT("ModuleUI", "Recompile_SlowTaskName", "Compiling {ModuleName}..."), Args ), bShowProgressDialog, bShowCancelButton );

	// Does the module have any UObject classes in it?  If so we'll use HotReload to recompile it.
	FModuleStatus ModuleStatus;
	if( ensure( FModuleManager::Get().QueryModule( ModuleName, ModuleStatus ) ) )
	{
		//@todo This is for content-only packages that show up in the
		// Module UI... don't crash when recompile is clicked
		if (FPaths::IsProjectFilePathSet())
		{
			if (ModuleStatus.bIsLoaded == false)
			{
				if (FPlatformFileManager::Get().GetPlatformFile().FileExists(*ModuleStatus.FilePath) == false)
				{
					UE_LOGF(LogTemp, Display, "Unable to recompile module %ls... Is it a content-only module?", *ModuleName.ToString());
					GWarn->EndSlowTask();
					return FReply::Handled();
				}
			}
		}

		TArray< UPackage* > PackagesToRebind;
		if( ModuleStatus.bIsLoaded )
		{
			const bool bIsHotReloadable = FModuleManager::Get().DoesLoadedModuleHaveUObjects( ModuleName );
			if( bIsHotReloadable )
			{
				// Is there a UPackage with the same name as this module?
				FString PotentialPackageName = FString(TEXT("/Script/")) + ModuleName.ToString();
				UPackage* Package = FindPackage( NULL, *PotentialPackageName);
				if( Package != NULL )
				{
					PackagesToRebind.Add( Package );
				}
			}
		}

		IHotReloadInterface& HotReloadSupport = FModuleManager::LoadModuleChecked<IHotReloadInterface>("HotReload");
		if( PackagesToRebind.Num() > 0 )
		{
			// Perform a hot reload
			ECompilationResult::Type Result = HotReloadSupport.RebindPackages(PackagesToRebind, EHotReloadFlags::WaitForCompletion, *GLog);
		}
		else
		{
			// Perform a regular unload, then reload
			const bool bRecompileSucceeded = HotReloadSupport.RecompileModule(ModuleName, *GLog, ERecompileModuleFlags::ReloadAfterRecompile | ERecompileModuleFlags::FailIfGeneratedCodeChanges);
		}
	}

	GWarn->EndSlowTask();

	return FReply::Handled();
}


EVisibility SModuleUI::FModuleListItem::GetVisibilityBasedOnLoadedAndShutdownableState() const
{
	if ( UE::IsSavingPackage() || IsGarbageCollecting() )
	{
		return EVisibility::Hidden;
	}

	const bool bIsHotReloadable = FModuleManager::Get().DoesLoadedModuleHaveUObjects(ModuleName);
	const bool bCanShutDown = ( FModuleManager::Get().IsModuleLoaded(ModuleName)
								&& !bIsHotReloadable 
								&& FModuleManager::Get().GetModule(ModuleName)->SupportsDynamicReloading() );

	return bCanShutDown ? EVisibility::Visible : EVisibility::Hidden;
}


EVisibility SModuleUI::FModuleListItem::GetVisibilityBasedOnReloadableState() const
{
	return GetVisibilityBasedOnLoadedAndShutdownableState();
};


EVisibility SModuleUI::FModuleListItem::GetVisibilityBasedOnRecompilableState() const
{
	if ( UE::IsSavingPackage() || IsGarbageCollecting() )
	{
		return EVisibility::Hidden;
	}

	if (!CanAccessSourceCode())
	{
		return EVisibility::Hidden;
	}

	const bool bIsHotReloadable = FModuleManager::Get().DoesLoadedModuleHaveUObjects(ModuleName);
	const bool bCanReload = ( !FModuleManager::Get().IsModuleLoaded(ModuleName)
							|| FModuleManager::Get().GetModule(ModuleName)->SupportsDynamicReloading()
							|| bIsHotReloadable );

	return bCanReload ? EVisibility::Visible : EVisibility::Hidden;
}


EVisibility SModuleUI::FModuleListItem::GetVisibilityBasedOnUnloadedState() const
{
	return FModuleManager::Get().IsModuleLoaded( ModuleName ) ? EVisibility::Hidden : EVisibility::Visible;
}


