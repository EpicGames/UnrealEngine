// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

class ITableRow;
class SSearchBox;
class STableViewBase;

template <typename ItemType> class SListView;

class SModuleUI : public SCompoundWidget
{

public:

	SLATE_BEGIN_ARGS( SModuleUI ){}
	SLATE_END_ARGS()


	/**
	 * Constructs this widget
	 *
	 * @param InArgs    Declaration from which to construct the widget
	 */
	void Construct(const FArguments& InArgs);

	/** SModuleUI destructor */
	~SModuleUI();


private:

	/** An item in the module list */
	struct FModuleListItem
	{
		/** Name of this module */
		FName ModuleName;

		/** Called when 'Load' is clicked in the UI */
		FReply OnLoadClicked();

		/** Called when 'Unload' is clicked in the UI */
		FReply OnUnloadClicked();

		/** Called when 'Reload' is clicked in the UI */
		FReply OnReloadClicked();

		/** Called when 'Recompile' is clicked in the UI */
		FReply OnRecompileClicked();

		/** @return Returns Visible for modules that are currently loaded and shutdown-able, otherwise Collapsed */
		EVisibility GetVisibilityBasedOnLoadedAndShutdownableState() const;

		/** @return Returns Visible for modules that are currently loaded and reload-able, otherwise Collapsed */
		EVisibility GetVisibilityBasedOnReloadableState() const;

		/** @return Returns Visible for modules that are either not yet loaded or can be shutdown, otherwise Collapsed */
		EVisibility GetVisibilityBasedOnRecompilableState() const;

		/** @return Returns Collapsed for modules that are currently loaded, otherwise Visible */
		EVisibility GetVisibilityBasedOnUnloadedState() const;

	};

	/**
	 * Given a data item, reutrns a widget to represent that item in the list view
	 *
	 * @param InItem      The list item to return a widget for
	 * @param OwnerTable  The TableView that this TableRow is being generated for
	 *
	 * @return	The widget to represent the specified list item	 
	 */
	TSharedRef<ITableRow> OnGenerateWidgetForModuleListView(TSharedPtr< FModuleListItem > InItem, const TSharedRef<STableViewBase>& OwnerTable);

	/**
	 * Called by the engine's module manager when a module is loaded, unloaded, or the list of known
	 * modules has changed
	 */
	void OnModulesChanged( FName ModuleThatChanged, EModuleChangeReason ReasonForChange );

	/**
	 * Updates our module list items
	 */
	void UpdateModuleListItems();

	/**
	 *  module list item filter 
	 */
	void OnFilterTextChanged(const FText& InFilterText);



private:

	typedef SListView< TSharedPtr< FModuleListItem > > SModuleListView;
	typedef TArray< TSharedPtr< FModuleListItem > > FModuleArray;

	/** List items for the module list */
	TArray< TSharedPtr< FModuleListItem > > ModuleListItems;

	/** List of all known modules */
	TSharedPtr< SModuleListView > ModuleListView;

	TSharedPtr< SSearchBox > ModuleNameSearchBox;
};
