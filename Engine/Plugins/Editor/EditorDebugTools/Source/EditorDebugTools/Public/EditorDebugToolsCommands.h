// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "EditorDebugToolsStyle.h"

class FEditorDebugToolsCommands : public TCommands<FEditorDebugToolsCommands>
{
public:

	FEditorDebugToolsCommands()
		: TCommands<FEditorDebugToolsCommands>(TEXT("EditorDebugTools"), NSLOCTEXT("Contexts", "EditorDebugTools", "EditorDebugTools Plugin"), NAME_None, FEditorDebugToolsStyle::GetStyleSetName())
	{
	}

	// TCommands<> interface
	virtual void RegisterCommands() override;

public:
	TSharedPtr< FUICommandInfo > OpenPluginWindow;
};
